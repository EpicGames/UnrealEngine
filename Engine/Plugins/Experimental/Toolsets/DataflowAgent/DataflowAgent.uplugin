{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DataflowAgent",
	"Description": "AI Agent toolset for editing Dataflow graphs",
	"Category": "Other",
	"EditorOnly": true,
	"NoRedist": true,
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"NoCode": false,
	"Modules": [
		{
			"Name": "DataflowAgent",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryCollectionPlugin",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
