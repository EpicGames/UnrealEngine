// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataflowAgent : ModuleRules
{
	public DataflowAgent(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bTreatAsEngineModule = true;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ToolsetRegistry",
				"DataflowCore",
				"DataflowEngine",
				"Projects",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"UnrealEd",
				"DataflowEditor",
				"AssetTools",
				"AssetRegistry",
				"Json",
				"JsonUtilities",
				"EditorSubsystem",
				"GraphEditor",
				"Kismet",
			}
		);
	}
}
