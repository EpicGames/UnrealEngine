// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowAgentModule.h"

#include "DataflowAgentToolset.h"
#include "Misc/CoreDelegates.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

DEFINE_LOG_CATEGORY(LogDataflowAgent);

#define LOCTEXT_NAMESPACE "FDataflowAgentModule"

void FDataflowAgentModule::StartupModule()
{
	UToolsetRegistry::RegisterToolsetClass(UDataflowAgentToolset::StaticClass());
}

void FDataflowAgentModule::ShutdownModule()
{
	UToolsetRegistry::UnregisterToolsetClass(UDataflowAgentToolset::StaticClass());
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDataflowAgentModule, DataflowAgent)
