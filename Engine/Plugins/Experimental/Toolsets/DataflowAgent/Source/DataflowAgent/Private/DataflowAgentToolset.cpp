// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowAgentToolset.h"

#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowAssetEditUtils.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "Dataflow/DataflowVariableNodes.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Editor.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphNode_Comment.h"
#include "IAssetTools.h"
#include "JsonObjectConverter.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/PackageName.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "ScopedTransaction.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

// ---------------------------------------------------------------------------
// Internal helpers
// ---------------------------------------------------------------------------
namespace DataflowAgentToolset
{
	/** Serialize any JSON value array to a compact string */
	static FString JsonArrayToString(const TArray<TSharedPtr<FJsonValue>>& Array)
	{
		FString Out;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
			TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Out);
		FJsonSerializer::Serialize(Array, Writer);
		return Out;
	}

	/** Serialize a JSON object to a compact string */
	static FString JsonObjectToString(const TSharedPtr<FJsonObject>& Obj)
	{
		FString Out;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
			TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Out);
		FJsonSerializer::Serialize(Obj.ToSharedRef(), Writer);
		return Out;
	}

	/** Try to parse a JSON object from a string; returns null on failure */
	static TSharedPtr<FJsonObject> ParseJsonObject(const FString& JsonStr)
	{
		TSharedPtr<FJsonObject> Obj;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonStr);
		FJsonSerializer::Deserialize(Reader, Obj);
		return Obj;
	}

	/**
	 * Find a UScriptStruct from a Dataflow type name like "FAddFloatsDataflowNode".
	 * UScriptStructs are registered without the leading "F".
	 */
	static UScriptStruct* FindNodeStruct(FName TypeName)
	{
		FString StructName = TypeName.ToString();
		// Strip leading "F" (USTRUCT convention)
		if (StructName.Len() > 1 && StructName[0] == TEXT('F'))
		{
			StructName = StructName.RightChop(1);
		}
		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			if ((*It)->GetName() == StructName)
			{
				return *It;
			}
		}
		return nullptr;
	}

	/** Build a JSON array describing the pins of a DataflowNode */
	static TArray<TSharedPtr<FJsonValue>> GetNodePinsJson(const FDataflowNode* Node, UE::Dataflow::FPin::EDirection Direction)
	{
		TArray<TSharedPtr<FJsonValue>> Result;
		if (!Node) return Result;

		for (const UE::Dataflow::FPin& Pin : Node->GetPins())
		{
			if (Pin.Direction != Direction) continue;
			TSharedPtr<FJsonObject> PinObj = MakeShared<FJsonObject>();
			PinObj->SetStringField(TEXT("name"), Pin.Name.ToString());
			PinObj->SetStringField(TEXT("type"), Pin.Type.ToString());
			Result.Add(MakeShared<FJsonValueObject>(PinObj));
		}
		return Result;
	}

	/** Build a JSON object describing a node's editable UPROPERTY fields */
	static TArray<TSharedPtr<FJsonValue>> GetNodePropertiesJson(const FDataflowNode* Node, UScriptStruct* NodeStruct)
	{
		TArray<TSharedPtr<FJsonValue>> Result;
		if (!Node || !NodeStruct) return Result;

		// Walk up the struct hierarchy to collect all properties
		for (TFieldIterator<FProperty> PropIt(NodeStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
		{
			const FProperty* Prop = *PropIt;
			if (!Prop->HasAnyPropertyFlags(CPF_Edit)) continue;

			// Skip the built-in base FDataflowNode properties (bOverrideColor, OverrideColor)
			if (Prop->GetOwnerStruct() == FDataflowNode::StaticStruct()) continue;

			TSharedPtr<FJsonObject> PropObj = MakeShared<FJsonObject>();
			PropObj->SetStringField(TEXT("name"), Prop->GetName());
			PropObj->SetStringField(TEXT("type"), Prop->GetCPPType());

			FString DefaultValue;
			Prop->ExportTextItem_Direct(DefaultValue, Prop->ContainerPtrToValuePtr<void>(Node), nullptr, nullptr, PPF_None);
			PropObj->SetStringField(TEXT("default"), DefaultValue);

			Result.Add(MakeShared<FJsonValueObject>(PropObj));
		}
		return Result;
	}

	/**
	 * Apply JSON properties to a DataflowNode using UScriptStruct reflection.
	 * Returns true if all specified properties were set successfully.
	 */
	static bool ApplyJsonToNode(FDataflowNode* Node, UScriptStruct* NodeStruct, const TSharedPtr<FJsonObject>& JsonParams)
	{
		if (!Node || !NodeStruct || !JsonParams.IsValid()) return false;

		bool bAllSucceeded = true;
		for (const auto& Pair : JsonParams->Values)
		{
			FProperty* Property = NodeStruct->FindPropertyByName(FName(*Pair.Key));
			if (!Property)
			{
				// Search in parent structs
				for (TFieldIterator<FProperty> PropIt(NodeStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
				{
					if ((*PropIt)->GetName().Equals(*Pair.Key, ESearchCase::IgnoreCase))
					{
						Property = *PropIt;
						break;
					}
				}
			}

			if (!Property)
			{
				UE_LOGF(LogTemp, Warning, "DataflowAgentToolset: Property '%ls' not found on node type", *Pair.Key);
				bAllSucceeded = false;
				continue;
			}

			void* PropertyValue = Property->ContainerPtrToValuePtr<void>(Node);
			if (!FJsonObjectConverter::JsonValueToUProperty(Pair.Value, Property, PropertyValue, 0, 0, false, nullptr))
			{
				UE_LOGF(LogTemp, Error, "DataflowAgentToolset : Failed to convert Json to property: %ls", *Pair.Key);
				return false;
			}

			//FString ValueStr;
			//if (Pair.Value->TryGetString(ValueStr))
			//{
			//	Property->ImportText_Direct(*ValueStr, Prop->ContainerPtrToValuePtr<void>(Node), nullptr, PPF_None);
			//}
			//else
			//{
			//	// For non-string JSON values, serialize to string first
			//	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
			//		TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&ValueStr);
			//	FJsonSerializer::Serialize({ Pair.Value.ToSharedRef() }, Writer);
			//	Property->ImportText_Direct(*ValueStr, Property->ContainerPtrToValuePtr<void>(Node), nullptr, PPF_None);
			//}
		}
		return bAllSucceeded;
	}

	/**
	 * Find a UClass from a name like "StaticMesh", "UStaticMesh", or "AMyActor".
	 * UClasses are registered without the leading "U" or "A".
	 */
	static UClass* FindClass(FName ClassName)
	{
		FString Name = ClassName.ToString();
		// Strip leading "U" or "A" (UCLASS convention)
		if (Name.Len() > 1 && (Name[0] == TEXT('U') || Name[0] == TEXT('A')))
		{
			Name = Name.RightChop(1);
		}
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if ((*It)->GetName() == Name)
			{
				return *It;
			}
		}
		return nullptr;
	}

	/** Get the owning UDataflow asset from an EdNode */
	static UDataflow* GetDataflowFromEdNode(const UDataflowEdNode* EdNode)
	{
		if (!EdNode) return nullptr;
		return UDataflow::GetDataflowAssetFromEdGraph(EdNode->GetGraph());
	}

	/**
	 * Resolve a Dataflow-compatible asset UClass from a user-supplied name.
	 * Accepts short names ("ChaosClothAsset"), prefixed names ("UChaosClothAsset"),
	 * or full path names ("/Script/ChaosClothAssetEngine.ChaosClothAsset").
	 * Validates the class implements IDataflowInstanceInterface and is not abstract/deprecated.
	 * Returns nullptr (and raises a script error) on failure.
	 */
	static UClass* ResolveDataflowCompatibleClass(const FString& ClassName)
	{
		if (ClassName.IsEmpty())
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("ClassName must be provided"));
			return nullptr;
		}

		// TryFindTypeSlow handles both full path names and short names without the U/A prefix
		// (UClass::GetName() omits the prefix). FindClass below covers the "UChaosClothAsset"
		// form where the caller included the prefix.
		UClass* Class = UClass::TryFindTypeSlow<UClass>(ClassName);
		if (!Class)
		{
			Class = FindClass(FName(*ClassName));
		}
		if (!Class)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Unknown class '%s'"), *ClassName));
			return nullptr;
		}

		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Class '%s' is abstract or deprecated"), *ClassName));
			return nullptr;
		}

		if (!Class->ImplementsInterface(UDataflowInstanceInterface::StaticClass()))
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Class '%s' is not Dataflow-compatible (does not implement IDataflowInstanceInterface)"),
				*ClassName));
			return nullptr;
		}

		return Class;
	}
}

// ---------------------------------------------------------------------------
// Graph
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::CreateGraph(const FString& Name, const FString& Path)
{
	if (Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Name parameter must be provided"));
		return {};
	}

	const FString TargetPath = Path.IsEmpty() ? TEXT("/Game") : Path;
	const FString FullPath = TargetPath / Name;

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	FString UniqueAssetName;
	FString PackageNameToUse;
	AssetToolsModule.Get().CreateUniqueAssetName(FullPath, FString(), PackageNameToUse, UniqueAssetName);
	const FString PackagePath = FPackageName::GetLongPackagePath(PackageNameToUse);

	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(UniqueAssetName, PackagePath, UDataflow::StaticClass(), nullptr);
	UDataflow* Graph = Cast<UDataflow>(NewAsset);
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Could not create Dataflow asset"));
		return {};
	}

	Graph->MarkPackageDirty();
	return Graph->GetPathName();
}

FString UDataflowAgentToolset::GetGraphStructure(const UDataflow* Graph)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return {};
	}

	TSharedPtr<const UE::Dataflow::FGraph> DataflowGraph = Graph->GetDataflow();
	if (!DataflowGraph.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Dataflow asset has no internal graph"));
		return {};
	}

	TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
	Root->SetStringField(TEXT("name"), Graph->GetFName().ToString());

	// Nodes
	TArray<TSharedPtr<FJsonValue>> NodesJson;
	for (const TSharedPtr<FDataflowNode>& Node : DataflowGraph->GetNodes())
	{
		if (!Node.IsValid()) continue;
		TSharedPtr<FJsonObject> NodeObj = MakeShared<FJsonObject>();
		NodeObj->SetStringField(TEXT("name"), Node->GetName().ToString());
		NodeObj->SetStringField(TEXT("type"), Node->GetType().ToString());
		NodeObj->SetStringField(TEXT("guid"), Node->GetGuid().ToString());

		// Position from the corresponding EdNode
		if (TObjectPtr<const UDataflowEdNode> EdNode = Graph->FindEdNodeByDataflowNodeGuid(Node->GetGuid()))
		{
			NodeObj->SetNumberField(TEXT("posX"), EdNode->NodePosX);
			NodeObj->SetNumberField(TEXT("posY"), EdNode->NodePosY);
		}

		// Pins
		NodeObj->SetArrayField(TEXT("inputPins"),
			DataflowAgentToolset::GetNodePinsJson(Node.Get(), UE::Dataflow::FPin::EDirection::INPUT));
		NodeObj->SetArrayField(TEXT("outputPins"),
			DataflowAgentToolset::GetNodePinsJson(Node.Get(), UE::Dataflow::FPin::EDirection::OUTPUT));

		NodesJson.Add(MakeShared<FJsonValueObject>(NodeObj));
	}
	Root->SetArrayField(TEXT("nodes"), NodesJson);

	// Connections
	TArray<TSharedPtr<FJsonValue>> ConnectionsJson;
	for (const UE::Dataflow::FLink& Link : DataflowGraph->GetConnections())
	{
		TSharedPtr<FJsonObject> LinkObj = MakeShared<FJsonObject>();

		// Resolve node guids to names where possible
		if (TSharedPtr<const FDataflowNode> FromNode = DataflowGraph->FindBaseNode(Link.OutputNode))
		{
			LinkObj->SetStringField(TEXT("fromNode"), FromNode->GetName().ToString());
		}
		else
		{
			LinkObj->SetStringField(TEXT("fromNodeGuid"), Link.OutputNode.ToString());
		}
		LinkObj->SetStringField(TEXT("fromPinGuid"), Link.Output.ToString());

		if (TSharedPtr<const FDataflowNode> ToNode = DataflowGraph->FindBaseNode(Link.InputNode))
		{
			LinkObj->SetStringField(TEXT("toNode"), ToNode->GetName().ToString());
		}
		else
		{
			LinkObj->SetStringField(TEXT("toNodeGuid"), Link.InputNode.ToString());
		}
		LinkObj->SetStringField(TEXT("toPinGuid"), Link.Input.ToString());

		// Resolve pin names from the connection objects
		if (TSharedPtr<const FDataflowNode> FromNode = DataflowGraph->FindBaseNode(Link.OutputNode))
		{
			if (const FDataflowOutput* Output = FromNode->FindOutput(Link.Output))
			{
				LinkObj->SetStringField(TEXT("fromPin"), Output->GetName().ToString());
			}
		}
		if (TSharedPtr<const FDataflowNode> ToNode = DataflowGraph->FindBaseNode(Link.InputNode))
		{
			if (const FDataflowInput* Input = ToNode->FindInput(Link.Input))
			{
				LinkObj->SetStringField(TEXT("toPin"), Input->GetName().ToString());
			}
		}

		ConnectionsJson.Add(MakeShared<FJsonValueObject>(LinkObj));
	}
	Root->SetArrayField(TEXT("connections"), ConnectionsJson);

	return DataflowAgentToolset::JsonObjectToString(Root);
}

// ---------------------------------------------------------------------------
// Node Types
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::ListNodeTypes(const bool bCommonOnly)
{
	UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance();
	if (!Factory)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Dataflow NodeFactory not available"));
		return {};
	}

	TArray<TSharedPtr<FJsonValue>> Result;
	for (const UE::Dataflow::FFactoryParameters& Params : Factory->RegisteredParameters())
	{
		if (!Params.IsValid()) continue;
		if (bCommonOnly && (Params.IsDeprecated() || Params.IsExperimental())) continue;

		TSharedPtr<FJsonObject> NodeObj = MakeShared<FJsonObject>();
		NodeObj->SetStringField(TEXT("typeName"),    Params.TypeName.ToString());
		NodeObj->SetStringField(TEXT("displayName"), Params.DisplayName.ToString());
		NodeObj->SetStringField(TEXT("category"),    Params.Category.ToString());
		if (!Params.ToolTip.IsEmpty())
		{
			NodeObj->SetStringField(TEXT("tooltip"), Params.ToolTip);
		}
		NodeObj->SetBoolField(TEXT("isDeprecated"), Params.bIsDeprecated);
		NodeObj->SetBoolField(TEXT("isExperimental"), Params.bIsExperimental);

		Result.Add(MakeShared<FJsonValueObject>(NodeObj));
	}

	return DataflowAgentToolset::JsonArrayToString(Result);
}

FString UDataflowAgentToolset::GetNodeTypeSchema(const FString& TypeName)
{
	if (TypeName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: TypeName must be provided"));
		return {};
	}

	UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance();
	if (!Factory)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Dataflow NodeFactory not available"));
		return {};
	}

	const UE::Dataflow::FFactoryParameters& Params = Factory->GetParameters(FName(*TypeName));
	if (!Params.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Error: Node type '%s' not found"), *TypeName));
		return {};
	}

	TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
	Schema->SetStringField(TEXT("typeName"),    Params.TypeName.ToString());
	Schema->SetStringField(TEXT("displayName"), Params.DisplayName.ToString());
	Schema->SetStringField(TEXT("category"),    Params.Category.ToString());
	Schema->SetStringField(TEXT("tooltip"),     Params.ToolTip);
	Schema->SetBoolField(TEXT("isDeprecated"), Params.bIsDeprecated);
	Schema->SetBoolField(TEXT("isExperimental"), Params.bIsExperimental);

	const FDataflowNode* DefaultNode = Params.DefaultNodeObject.Get();
	Schema->SetArrayField(TEXT("inputPins"),
		DataflowAgentToolset::GetNodePinsJson(DefaultNode, UE::Dataflow::FPin::EDirection::INPUT));
	Schema->SetArrayField(TEXT("outputPins"),
		DataflowAgentToolset::GetNodePinsJson(DefaultNode, UE::Dataflow::FPin::EDirection::OUTPUT));

	// Editable properties
	UScriptStruct* NodeStruct = DataflowAgentToolset::FindNodeStruct(Params.TypeName);
	if (NodeStruct && DefaultNode)
	{
		Schema->SetArrayField(TEXT("properties"),
			DataflowAgentToolset::GetNodePropertiesJson(DefaultNode, NodeStruct));
	}

	return DataflowAgentToolset::JsonObjectToString(Schema);
}

// ---------------------------------------------------------------------------
// Nodes
// ---------------------------------------------------------------------------

UDataflowEdNode* UDataflowAgentToolset::AddNode(
	UDataflow* Graph,
	const FString& TypeName,
	const FString& NodeName,
	const FString& JsonParams,
	const int32 X,
	const int32 Y)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return nullptr;
	}
	if (TypeName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: TypeName must be provided"));
		return nullptr;
	}
	if (NodeName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: NodeName must be provided"));
		return nullptr;
	}

	UE::Dataflow::FNodeFactory* Factory = UE::Dataflow::FNodeFactory::GetInstance();
	if (!Factory)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Dataflow NodeFactory not available"));
		return nullptr;
	}

	if (!Factory->GetParameters(FName(*TypeName)).IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Error: Node type '%s' not found"), *TypeName));
		return nullptr;
	}

	FScopedTransaction Transaction(INVTEXT("AddDataflowNodeToolCall"));

	UDataflowEdNode* NewEdNode = UE::Dataflow::FEditAssetUtils::AddNewNode(
		Graph, FVector2D(X, Y), FName(*NodeName), FName(*TypeName), /*FromPin=*/nullptr);

	if (!NewEdNode)
	{
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Failed to create node of type '%s'"), *TypeName));
		return nullptr;
	}

	// Apply JSON parameters if provided
	if (!JsonParams.IsEmpty())
	{
		TSharedPtr<FJsonObject> ParamsObj = DataflowAgentToolset::ParseJsonObject(JsonParams);
		if (!ParamsObj.IsValid())
		{
			Transaction.Cancel();
			UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Failed to parse JsonParams"));
			return nullptr;
		}

		TSharedPtr<FDataflowNode> DataflowNode = NewEdNode->GetDataflowNode();
		UScriptStruct* NodeStruct = DataflowAgentToolset::FindNodeStruct(FName(*TypeName));
		if (NodeStruct && DataflowNode.IsValid())
		{
			DataflowAgentToolset::ApplyJsonToNode(DataflowNode.Get(), NodeStruct, ParamsObj);
		}
	}

	// Special case for variables nodes as we need to refresh the node to properly connect to the existing variable
	if (TSharedPtr<FDataflowNode> DataflowNode = NewEdNode->GetDataflowNode())
	{
		if (FGetDataflowVariableNode* VariableNode = DataflowNode->AsType<FGetDataflowVariableNode>())
		{
			VariableNode->SetVariable(Graph, VariableNode->GetVariableName());
		}
	}

	Graph->MarkPackageDirty();
	return NewEdNode;
}

void UDataflowAgentToolset::UpdateNode(UDataflowEdNode* Node, const FString& JsonParams)
{
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No EdNode provided"));
		return;
	}
	if (JsonParams.IsEmpty())
	{
		return;
	}

	TSharedPtr<FDataflowNode> DataflowNode = Node->GetDataflowNode();
	if (!DataflowNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: EdNode has no associated Dataflow node"));
		return;
	}

	TSharedPtr<FJsonObject> ParamsObj = DataflowAgentToolset::ParseJsonObject(JsonParams);
	if (!ParamsObj.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Failed to parse JsonParams"));
		return;
	}

	UScriptStruct* NodeStruct = DataflowAgentToolset::FindNodeStruct(DataflowNode->GetType());
	if (!NodeStruct)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Could not find UScriptStruct for node type '%s'"),
			*DataflowNode->GetType().ToString()));
		return;
	}

	FScopedTransaction Transaction(INVTEXT("UpdateDataflowNodeToolCall"));
	Node->Modify();

	if (!DataflowAgentToolset::ApplyJsonToNode(DataflowNode.Get(), NodeStruct, ParamsObj))
	{
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: One or more properties could not be set"));
		return;
	}

	UDataflow* DataflowAsset = DataflowAgentToolset::GetDataflowFromEdNode(Node);
	if (DataflowAsset)
	{
		DataflowAsset->RefreshEdNode(Node);
		DataflowAsset->MarkPackageDirty();
	}
}

FString UDataflowAgentToolset::GetNodeInfo(UDataflowEdNode* Node)
{
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No EdNode provided"));
		return {};
	}

	TSharedPtr<const FDataflowNode> DataflowNode = Node->GetDataflowNode();

	TSharedPtr<FJsonObject> Info = MakeShared<FJsonObject>();
	Info->SetStringField(TEXT("name"),    Node->GetName());
	Info->SetStringField(TEXT("guid"),    Node->GetDataflowNodeGuid().ToString());
	Info->SetNumberField(TEXT("posX"),    Node->NodePosX);
	Info->SetNumberField(TEXT("posY"),    Node->NodePosY);

	if (DataflowNode.IsValid())
	{
		Info->SetStringField(TEXT("type"),        DataflowNode->GetType().ToString());
		Info->SetStringField(TEXT("displayName"), DataflowNode->GetDisplayName().ToString());
		Info->SetStringField(TEXT("category"),    DataflowNode->GetCategory().ToString());

		Info->SetArrayField(TEXT("inputPins"),
			DataflowAgentToolset::GetNodePinsJson(DataflowNode.Get(), UE::Dataflow::FPin::EDirection::INPUT));
		Info->SetArrayField(TEXT("outputPins"),
			DataflowAgentToolset::GetNodePinsJson(DataflowNode.Get(), UE::Dataflow::FPin::EDirection::OUTPUT));

		// Current property values
		UScriptStruct* NodeStruct = DataflowAgentToolset::FindNodeStruct(DataflowNode->GetType());
		if (NodeStruct)
		{
			TSharedPtr<FJsonObject> Props = MakeShared<FJsonObject>();
			for (TFieldIterator<FProperty> PropIt(NodeStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				const FProperty* Prop = *PropIt;
				if (!Prop->HasAnyPropertyFlags(CPF_Edit)) continue;
				if (Prop->GetOwnerStruct() == FDataflowNode::StaticStruct()) continue;

				FString Value;
				Prop->ExportTextItem_Direct(Value, Prop->ContainerPtrToValuePtr<void>(DataflowNode.Get()), nullptr, nullptr, PPF_None);
				Props->SetStringField(Prop->GetName(), Value);
			}
			Info->SetObjectField(TEXT("properties"), Props);
		}
	}

	return DataflowAgentToolset::JsonObjectToString(Info);
}

void UDataflowAgentToolset::RepositionNode(UDataflowEdNode* Node, const int32 X, const int32 Y)
{
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No EdNode provided"));
		return;
	}

	FScopedTransaction Transaction(INVTEXT("RepositionDataflowNodeToolCall"));
	Node->Modify();
	Node->NodePosX = X;
	Node->NodePosY = Y;

	UDataflow* DataflowAsset = DataflowAgentToolset::GetDataflowFromEdNode(Node);
	if (DataflowAsset)
	{
		DataflowAsset->MarkPackageDirty();
	}
}

void UDataflowAgentToolset::RemoveNode(UDataflow* Graph, UDataflowEdNode* Node)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return;
	}
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No EdNode provided"));
		return;
	}

	// Verify the node belongs to this graph
	if (UDataflow::GetDataflowAssetFromEdGraph(Node->GetGraph()) != Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Node does not belong to the provided graph"));
		return;
	}

	TArray<UEdGraphNode*> NodesToDelete = { Node };
	UE::Dataflow::FEditAssetUtils::DeleteNodes(Graph, NodesToDelete);
}

// ---------------------------------------------------------------------------
// Connections
// ---------------------------------------------------------------------------

bool UDataflowAgentToolset::ConnectNodePins(
	UDataflowEdNode* FromNode, const FString& FromPin,
	UDataflowEdNode* ToNode,   const FString& ToPin)
{
	if (!FromNode)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: FromNode is null"));
		return false;
	}
	if (!ToNode)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: ToNode is null"));
		return false;
	}

	UDataflow* FromAsset = DataflowAgentToolset::GetDataflowFromEdNode(FromNode);
	UDataflow* ToAsset   = DataflowAgentToolset::GetDataflowFromEdNode(ToNode);
	if (!FromAsset || FromAsset != ToAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Both nodes must belong to the same Dataflow graph"));
		return false;
	}

	TSharedPtr<FDataflowNode> SourceDFNode = FromNode->GetDataflowNode();
	TSharedPtr<FDataflowNode> TargetDFNode = ToNode->GetDataflowNode();
	if (!SourceDFNode.IsValid() || !TargetDFNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Could not retrieve internal Dataflow nodes"));
		return false;
	}

	FDataflowOutput* Output = SourceDFNode->FindOutput(FName(*FromPin));
	if (!Output)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Node '%s' has no output pin named '%s'"),
			*FromNode->GetName(), *FromPin));
		return false;
	}

	FDataflowInput* Input = TargetDFNode->FindInput(FName(*ToPin));
	if (!Input)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Node '%s' has no input pin named '%s'"),
			*ToNode->GetName(), *ToPin));
		return false;
	}

	TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = FromAsset->GetDataflow();
	if (!DataflowGraph.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Dataflow graph is invalid"));
		return false;
	}

	if (!DataflowGraph->CanConnect(*Output, *Input))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Cannot connect pin '%s' on '%s' to pin '%s' on '%s' (type incompatibility)"),
			*FromPin, *FromNode->GetName(), *ToPin, *ToNode->GetName()));
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("ConnectDataflowPinsToolCall"));
	FromAsset->Modify();

	const bool bConnected = DataflowGraph->Connect(*Output, *Input);
	if (!bConnected)
	{
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Failed to connect '%s'.'%s' -> '%s'.'%s'"),
			*FromNode->GetName(), *FromPin, *ToNode->GetName(), *ToPin));
		return false;
	}

	// Sync the visual EdGraph pins
	FromAsset->RefreshEdNode(FromNode);
	FromAsset->RefreshEdNode(ToNode);
	FromAsset->MarkPackageDirty();

	return true;
}

void UDataflowAgentToolset::DisconnectNodePins(
	UDataflowEdNode* FromNode, const FString& FromPin,
	UDataflowEdNode* ToNode,   const FString& ToPin)
{
	if (!FromNode)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: FromNode is null"));
		return;
	}
	if (!ToNode)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: ToNode is null"));
		return;
	}

	UDataflow* FromAsset = DataflowAgentToolset::GetDataflowFromEdNode(FromNode);
	UDataflow* ToAsset   = DataflowAgentToolset::GetDataflowFromEdNode(ToNode);
	if (!FromAsset || FromAsset != ToAsset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Both nodes must belong to the same Dataflow graph"));
		return;
	}

	TSharedPtr<FDataflowNode> SourceDFNode = FromNode->GetDataflowNode();
	TSharedPtr<FDataflowNode> TargetDFNode = ToNode->GetDataflowNode();
	if (!SourceDFNode.IsValid() || !TargetDFNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Could not retrieve internal Dataflow nodes"));
		return;
	}

	FDataflowOutput* Output = SourceDFNode->FindOutput(FName(*FromPin));
	if (!Output)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Node '%s' has no output pin named '%s'"),
			*FromNode->GetName(), *FromPin));
		return;
	}

	FDataflowInput* Input = TargetDFNode->FindInput(FName(*ToPin));
	if (!Input)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Node '%s' has no input pin named '%s'"),
			*ToNode->GetName(), *ToPin));
		return;
	}

	TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = FromAsset->GetDataflow();
	if (!DataflowGraph.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(INVTEXT("DisconnectDataflowPinsToolCall"));
	FromAsset->Modify();

	DataflowGraph->Disconnect(Output, Input);

	// Sync the visual EdGraph pins
	FromAsset->RefreshEdNode(FromNode);
	FromAsset->RefreshEdNode(ToNode);
	FromAsset->MarkPackageDirty();
}

// ---------------------------------------------------------------------------
// Variables
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::ListVariables(const UDataflow* Graph)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return {};
	}

	const UPropertyBag* BagStruct = Graph->Variables.GetPropertyBagStruct();
	if (!BagStruct)
	{
		return DataflowAgentToolset::JsonArrayToString({});
	}

	TArray<TSharedPtr<FJsonValue>> Result;
	for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
	{
		TSharedPtr<FJsonObject> VarObj = MakeShared<FJsonObject>();
		VarObj->SetStringField(TEXT("name"), Desc.Name.ToString());

		const UEnum* TypeEnum = StaticEnum<EPropertyBagPropertyType>();
		VarObj->SetStringField(TEXT("type"), TypeEnum ? TypeEnum->GetNameStringByValue(static_cast<int64>(Desc.ValueType)) : FString::FromInt(static_cast<int32>(Desc.ValueType)));

		if (Desc.ValueTypeObject)
		{
			VarObj->SetStringField(TEXT("valueTypeName"), Desc.ValueTypeObject->GetName());
		}

		TValueOrError<FString, EPropertyBagResult> ValueResult = Graph->Variables.GetValueSerializedString(Desc.Name);
		if (ValueResult.HasValue())
		{
			VarObj->SetStringField(TEXT("value"), ValueResult.GetValue());
		}

		Result.Add(MakeShared<FJsonValueObject>(VarObj));
	}

	return DataflowAgentToolset::JsonArrayToString(Result);
}

bool UDataflowAgentToolset::AddVariable(UDataflow* Graph, const FString& Name, const FString& Type)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return false;
	}
	if (Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Name must be provided"));
		return false;
	}
	if (Type.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Type must be provided"));
		return false;
	}

	static const TMap<FString, EPropertyBagPropertyType> PrimitiveTypeMap =
	{
		{ TEXT("Bool"),   EPropertyBagPropertyType::Bool   },
		{ TEXT("Int32"),  EPropertyBagPropertyType::Int32  },
		{ TEXT("Int64"),  EPropertyBagPropertyType::Int64  },
		{ TEXT("Float"),  EPropertyBagPropertyType::Float  },
		{ TEXT("Double"), EPropertyBagPropertyType::Double },
		{ TEXT("Name"),   EPropertyBagPropertyType::Name   },
		{ TEXT("String"), EPropertyBagPropertyType::String },
	};

	EPropertyBagPropertyType BagType  = EPropertyBagPropertyType::None;
	const UObject*           TypeObj  = nullptr;

	if (const EPropertyBagPropertyType* Primitive = PrimitiveTypeMap.Find(Type))
	{
		BagType = *Primitive;
	}
	else if (Type.StartsWith(TEXT("Object:")))
	{
		// "Object:ClassName" or "Object:UMyClass"
		const FString ClassName = Type.RightChop(7); // strip "Object:"
		UClass* FoundClass = DataflowAgentToolset::FindClass(FName(*ClassName));
		if (!FoundClass)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Error: Could not find UClass '%s'"), *ClassName));
			return false;
		}
		BagType = EPropertyBagPropertyType::Object;
		TypeObj = FoundClass;
	}
	else
	{
		// Treat as UScriptStruct name (e.g. "Vector", "FVector", "Transform", "FTransform")
		UScriptStruct* FoundStruct = DataflowAgentToolset::FindNodeStruct(FName(*Type));
		if (!FoundStruct)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Error: Unknown type '%s'. For primitives use: Bool, Int32, Int64, Float, Double, Name, String. ")
				TEXT("For structs pass the struct name (e.g. Vector, Transform, Rotator). ")
				TEXT("For objects use 'Object:ClassName'."), *Type));
			return false;
		}
		BagType = EPropertyBagPropertyType::Struct;
		TypeObj = FoundStruct;
	}

	const FName VarName(*Name);
	if (Graph->Variables.FindPropertyDescByName(VarName) != nullptr)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Variable '%s' already exists on this graph"), *Name));
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("AddDataflowVariableToolCall"));
	Graph->Modify();

	Graph->Variables.AddProperties({ FPropertyBagPropertyDesc(VarName, BagType, TypeObj) });
	Graph->MarkPackageDirty();
	return true;
}

void UDataflowAgentToolset::RemoveVariable(UDataflow* Graph, const FString& Name)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return;
	}
	if (Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Name must be provided"));
		return;
	}

	const FName VarName(*Name);
	if (Graph->Variables.FindPropertyDescByName(VarName) == nullptr)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Variable '%s' not found on this graph"), *Name));
		return;
	}

	FScopedTransaction Transaction(INVTEXT("RemoveDataflowVariableToolCall"));
	Graph->Modify();

	Graph->Variables.RemovePropertyByName(VarName);
	Graph->MarkPackageDirty();
}

bool UDataflowAgentToolset::SetVariable(UDataflow* Graph, const FString& Name, const FString& Value)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return false;
	}
	if (Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Name must be provided"));
		return false;
	}

	const FName VarName(*Name);
	if (Graph->Variables.FindPropertyDescByName(VarName) == nullptr)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Variable '%s' not found on this graph"), *Name));
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("SetDataflowVariableToolCall"));
	Graph->Modify();

	const EPropertyBagResult SetResult = Graph->Variables.SetValueSerializedString(VarName, Value);
	if (SetResult != EPropertyBagResult::Success)
	{
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: Failed to set variable '%s' to '%s'"), *Name, *Value));
		return false;
	}

	Graph->MarkPackageDirty();
	return true;
}

// ---------------------------------------------------------------------------
// Comments
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::AddCommentBox(
	UDataflow* Graph,
	const TArray<UDataflowEdNode*>& Nodes,
	const FString& Comment,
	FLinearColor Color)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return {};
	}
	if (Nodes.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Nodes array must not be empty"));
		return {};
	}

	// Compute bounding box around provided nodes
	int32 MinX = INT_MAX, MinY = INT_MAX, MaxX = INT_MIN, MaxY = INT_MIN;
	for (const UDataflowEdNode* Node : Nodes)
	{
		if (!Node) continue;
		const int32 NodeW = 200;
		const int32 NodeH = 100;
		MinX = FMath::Min(MinX, Node->NodePosX);
		MinY = FMath::Min(MinY, Node->NodePosY);
		MaxX = FMath::Max(MaxX, Node->NodePosX + NodeW);
		MaxY = FMath::Max(MaxY, Node->NodePosY + NodeH);
	}
	if (MinX > MaxX || MinY > MaxY)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Invalid bounds for nodes"));
		return {};
	}

	const int32 Padding = 50;
	MinX -= Padding; MinY -= Padding;
	MaxX += Padding; MaxY += Padding;

	FScopedTransaction Transaction(INVTEXT("AddDataflowCommentBoxToolCall"));

	UEdGraphNode* CommentBase = UE::Dataflow::FEditAssetUtils::AddNewComment(
		Graph, FVector2D(MinX, MinY), FVector2D(MaxX - MinX, MaxY - MinY));

	if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(CommentBase))
	{
		CommentNode->Modify();
		CommentNode->NodeComment = Comment;
		CommentNode->CommentColor = Color;
		Graph->MarkPackageDirty();
		return CommentNode->NodeGuid.ToString();
	}

	Transaction.Cancel();
	UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Failed to create comment node"));
	return {};
}

void UDataflowAgentToolset::RemoveCommentBox(UDataflow* Graph, const FString& CommentId)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: No Dataflow asset provided"));
		return;
	}
	if (CommentId.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: CommentId must be provided"));
		return;
	}

	// Find the comment node by GUID
	UEdGraphNode_Comment* FoundComment = nullptr;
	for (UEdGraphNode* Node : Graph->Nodes)
	{
		if (UEdGraphNode_Comment* Comment = Cast<UEdGraphNode_Comment>(Node))
		{
			if (Comment->NodeGuid.ToString() == CommentId)
			{
				FoundComment = Comment;
				break;
			}
		}
	}

	if (!FoundComment)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Error: No comment node found with ID '%s'"), *CommentId));
		return;
	}

	FScopedTransaction Transaction(INVTEXT("RemoveDataflowCommentBoxToolCall"));
	Graph->Modify();
	Graph->RemoveNode(FoundComment, /*bBreakLinks=*/false);
	Graph->MarkPackageDirty();
}

// ---------------------------------------------------------------------------
// Assets
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::ListDataflowCompatibleAssetTypes()
{
	TArray<TSharedPtr<FJsonValue>> Result;
	const UClass* const Interface = UDataflowInstanceInterface::StaticClass();

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* const Class = *It;
		if (Class == Interface) continue;
		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Interface)) continue;
		if (!Class->ImplementsInterface(Interface)) continue;

		TSharedPtr<FJsonObject> Entry = MakeShared<FJsonObject>();
		Entry->SetStringField(TEXT("className"), Class->GetName());
		Entry->SetStringField(TEXT("displayName"), Class->GetDisplayNameText().ToString());
		Entry->SetStringField(TEXT("modulePath"), Class->GetPathName());
		Result.Add(MakeShared<FJsonValueObject>(Entry));
	}

	return DataflowAgentToolset::JsonArrayToString(Result);
}

namespace DataflowAgentToolset
{
	/**
	 * Shared implementation for CreateDataflowCompatibleAsset and
	 * CreateDataflowCompatibleAssetFromTemplate. Pass nullptr for TemplatePath when
	 * no template is desired (the embedded graph is still created, empty).
	 */
	static FString CreateDataflowCompatibleAssetImpl(
		const FString& ClassName,
		const FString& Name,
		const FString& Path,
		const FString* TemplatePath)
	{
		if (Name.IsEmpty())
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("Name parameter must be provided"));
			return {};
		}

		UClass* const Class = ResolveDataflowCompatibleClass(ClassName);
		if (!Class)
		{
			return {};
		}

		const FString TargetPath = Path.IsEmpty() ? TEXT("/Game") : Path;
		const FString FullPath = TargetPath / Name;

		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		FString UniquePackageName;
		FString UniqueAssetName;
		AssetToolsModule.Get().CreateUniqueAssetName(FullPath, FString(), UniquePackageName, UniqueAssetName);

		UPackage* const Package = CreatePackage(*UniquePackageName);
		if (!Package)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Could not create package '%s'"), *UniquePackageName));
			return {};
		}

		// FactoryCreateNew requires a non-null TemplatePath pointer to take the
		// embedded-Dataflow path (see AssetDefinition_DataflowAsset.cpp:650). An
		// empty string falls through to a fresh empty graph; a non-empty string
		// loads and duplicates the template.
		const FString EmptyTemplatePath;
		const FString* const PathToUse = TemplatePath ? TemplatePath : &EmptyTemplatePath;
		constexpr EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;

		UObject* const NewAsset = UE::DataflowAssetDefinitionHelpers::FactoryCreateNew(
			Class,
			Package,
			FName(*UniqueAssetName),
			Flags,
			PathToUse,
			/*bEmbedDataflow=*/true,
			/*AssetPrefix=*/nullptr);

		if (!NewAsset)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Could not create asset '%s' of class '%s'"), *UniqueAssetName, *ClassName));
			return {};
		}

		FAssetRegistryModule::AssetCreated(NewAsset);
		NewAsset->MarkPackageDirty();
		return NewAsset->GetPathName();
	}
}

FString UDataflowAgentToolset::CreateDataflowCompatibleAsset(
	const FString& ClassName,
	const FString& Name,
	const FString& Path)
{
	return DataflowAgentToolset::CreateDataflowCompatibleAssetImpl(ClassName, Name, Path, /*TemplatePath=*/nullptr);
}

FString UDataflowAgentToolset::CreateDataflowCompatibleAssetFromTemplate(
	const FString& ClassName,
	const FString& Name,
	const FString& Path,
	const FString& TemplateId)
{
	return DataflowAgentToolset::CreateDataflowCompatibleAssetImpl(ClassName, Name, Path, &TemplateId);
}

// ---------------------------------------------------------------------------
// Templates
// ---------------------------------------------------------------------------

FString UDataflowAgentToolset::ListDataflowTemplatesForAssetClass(
	const FString& ClassName,
	const bool bIncludeBlank)
{
	UClass* const Class = DataflowAgentToolset::ResolveDataflowCompatibleClass(ClassName);
	if (!Class)
	{
		return {};
	}

	const TArray<FDataflowTemplateOption> Options =
		FDataflowTemplateRegistry::Get().GetTemplateOptions(Class, bIncludeBlank);

	TArray<TSharedPtr<FJsonValue>> Result;
	Result.Reserve(Options.Num());
	for (const FDataflowTemplateOption& Option : Options)
	{
		TSharedPtr<FJsonObject> Entry = MakeShared<FJsonObject>();
		Entry->SetStringField(TEXT("templateId"), Option.TemplateId.ToString());
		Entry->SetStringField(TEXT("displayName"), Option.DisplayName.ToString());
		Entry->SetStringField(TEXT("tooltip"), Option.Tooltip.ToString());
		Result.Add(MakeShared<FJsonValueObject>(Entry));
	}

	return DataflowAgentToolset::JsonArrayToString(Result);
}

bool UDataflowAgentToolset::AssignDataflowTemplate(UObject* Asset, const FString& TemplateId)
{
	if (!Asset)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Asset parameter must be provided"));
		return false;
	}

	if (!Asset->GetClass()->ImplementsInterface(UDataflowInstanceInterface::StaticClass()))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Asset '%s' is not Dataflow-compatible (does not implement IDataflowInstanceInterface)"),
			*Asset->GetName()));
		return false;
	}

	// Load the template if a non-empty id was supplied. An empty id means "blank graph",
	// matching the picker's NAME_None convention (AssetDefinition_DataflowAsset.cpp:255).
	UDataflow* Source = nullptr;
	if (!TemplateId.IsEmpty())
	{
		Source = LoadObject<UDataflow>(nullptr, *TemplateId);
		if (!Source)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("Could not load Dataflow template '%s'"), *TemplateId));
			return false;
		}
	}

	FScopedTransaction Transaction(INVTEXT("AssignDataflowTemplateToolCall"));
	Asset->Modify();

	// Mirror AssetDefinition_DataflowAsset.cpp:259-273: duplicate the template into the
	// asset (or create a fresh empty graph), strip standalone flags, set transactional,
	// and wire it up via the InstanceUtils helper.
	UDataflow* const Embedded = Source
		? DuplicateObject<UDataflow>(Source, Asset, TEXT("EmbeddedDataflow"))
		: NewObject<UDataflow>(Asset, UDataflow::StaticClass(), TEXT("EmbeddedDataflow"), RF_Transactional);

	if (!Embedded)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to create embedded Dataflow"));
		return false;
	}

	if (Source)
	{
		Embedded->ClearFlags(RF_Public | RF_Standalone);
		Embedded->SetFlags(RF_Transactional);
	}

	if (!UE::Dataflow::InstanceUtils::SetDataflowAssetOnObject(Embedded, Asset))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to assign embedded Dataflow to asset"));
		return false;
	}

	Asset->MarkPackageDirty();
	return true;
}
