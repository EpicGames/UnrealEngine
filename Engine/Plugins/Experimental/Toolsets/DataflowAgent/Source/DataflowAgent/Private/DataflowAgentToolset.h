// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowAgentCustomTypes.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "DataflowAgentToolset.generated.h"

class UDataflow;
class UDataflowEdNode;

/**
 * Dataflow Agent Toolset - exposes Dataflow graph editing operations as AI agent tools.
 */
UCLASS(BlueprintType)
class UDataflowAgentToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	// Graph -----------------------------------------------------------------------------------------------------------

	/**
	 * Creates a new saved Dataflow graph asset.
	 *
	 * @param Name - Name for the new Dataflow asset (e.g., "DF_Simulation")
	 * @param Path - Content folder path where the asset should be created. Defaults to "/Game/Dataflow"
	 * @return The full asset path of the created graph, or empty string on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Graph")
	static FString CreateGraph(
		const FString& Name,
		const FString& Path);

	/**
	 * Returns the complete structure of a Dataflow graph including all nodes and connections.
	 *
	 * @param Graph - The Dataflow asset to inspect
	 * @return JSON string with graph structure (nodes + connections), or error message on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Graph")
	static FString GetGraphStructure(const UDataflow* Graph);

	// Node Types ------------------------------------------------------------------------------------------------------

	/**
	 * Returns a JSON list of all registered Dataflow node types.
	 *
	 * @param bCommonOnly - When true, only return non-deprecated non-experimental nodes. Default is true.
	 * @return JSON array of node type names
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|NodeTypes")
	static FString ListNodeTypes(const bool bCommonOnly = true);

	/**
	 * Returns the schema for a Dataflow node type including its input/output pins
	 * and editable UPROPERTY parameters.
	 *
	 * @param TypeName - The node type name (e.g., "FAddFloatsDataflowNode")
	 * @return JSON object with name, category, tooltip, inputPins, outputPins, and properties
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|NodeTypes")
	static FString GetNodeTypeSchema(const FString& TypeName);

	// Nodes -----------------------------------------------------------------------------------------------------------

	/**
	 * Adds a node of the given type to the Dataflow graph.
	 *
	 * @param Graph      - The Dataflow asset to add the node to
	 * @param TypeName   - The node type name (e.g., "FAddFloatsDataflowNode")
	 * @param NodeName   - Unique name for the node within this graph
	 * @param JsonParams - Optional JSON object of property overrides (e.g., {"Value": 3.14})
	 * @param X          - X position in the graph editor (default 0, typical node width ~200)
	 * @param Y          - Y position in the graph editor (default 0, typical node height ~100)
	 * @return The newly created UDataflowEdNode, or nullptr on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Nodes")
	static UDataflowEdNode* AddNode(
		UDataflow* Graph,
		const FString& TypeName,
		const FString& NodeName,
		const FString& JsonParams = TEXT(""),
		const int32 X = 0,
		const int32 Y = 0);

	/**
	 * Updates an existing node's editable properties via JSON.
	 *
	 * @param Node       - The EdNode to modify
	 * @param JsonParams - JSON object of property overrides (e.g., {"Value": 3.14})
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Nodes")
	static void UpdateNode(UDataflowEdNode* Node, const FString& JsonParams);

	/**
	 * Returns information about a node as a JSON object (name, type, position, pins).
	 *
	 * @param Node - The EdNode to query
	 * @return JSON string with node info, or error message on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Nodes")
	static FString GetNodeInfo(UDataflowEdNode* Node);

	/**
	 * Moves a node to a new position in the graph editor.
	 *
	 * @param Node - The EdNode to reposition
	 * @param X    - New X position
	 * @param Y    - New Y position
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Nodes")
	static void RepositionNode(UDataflowEdNode* Node, const int32 X, const int32 Y);

	/**
	 * Removes a node and all its connections from the Dataflow graph.
	 *
	 * @param Graph - The Dataflow asset containing the node
	 * @param Node  - The EdNode to remove
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Nodes")
	static void RemoveNode(UDataflow* Graph, UDataflowEdNode* Node);

	// Connections -----------------------------------------------------------------------------------------------------

	/**
	 * Connects an output pin of one node to an input pin of another.
	 *
	 * @param FromNode - The source EdNode
	 * @param FromPin  - Name of the output pin on the source node
	 * @param ToNode   - The destination EdNode
	 * @param ToPin    - Name of the input pin on the destination node
	 * @return true on success, false on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Connections")
	static bool ConnectNodePins(
		UDataflowEdNode* FromNode, const FString& FromPin,
		UDataflowEdNode* ToNode,   const FString& ToPin);

	/**
	 * Removes the connection between two node pins.
	 *
	 * @param FromNode - The source EdNode
	 * @param FromPin  - Name of the output pin on the source node
	 * @param ToNode   - The destination EdNode
	 * @param ToPin    - Name of the input pin on the destination node
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Connections")
	static void DisconnectNodePins(
		UDataflowEdNode* FromNode, const FString& FromPin,
		UDataflowEdNode* ToNode,   const FString& ToPin);

	// Variables -------------------------------------------------------------------------------------------------------

	/**
	 * Returns all variables defined on the Dataflow graph as a JSON array.
	 * Each entry contains "name", "type", and "value" fields.
	 *
	 * @param Graph - The Dataflow asset to inspect
	 * @return JSON array of variable descriptors, or empty string on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Variables")
	static FString ListVariables(const UDataflow* Graph);

	/**
	 * Adds a new variable to the Dataflow graph.
	 *
	 * Supported type strings:
	 *   Primitives : "Bool", "Int32", "Int64", "Float", "Double", "Name", "String"
	 *   Structs    : UScriptStruct name with or without the "F" prefix
	 *                e.g. "Vector", "FVector", "Transform", "FTransform", "Rotator", "LinearColor"
	 *   Objects    : "Object:<ClassName>" where ClassName is with or without the "U"/"A" prefix
	 *                e.g. "Object:StaticMesh", "Object:USkeletalMesh"
	 *
	 * @param Graph - The Dataflow asset to modify
	 * @param Name  - Unique name for the new variable
	 * @param Type  - Type string as described above
	 * @return true on success, false if the variable already exists or the type cannot be resolved
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Variables")
	static bool AddVariable(UDataflow* Graph, const FString& Name, const FString& Type);

	/**
	 * Removes a variable from the Dataflow graph.
	 *
	 * @param Graph - The Dataflow asset to modify
	 * @param Name  - Name of the variable to remove
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Variables")
	static void RemoveVariable(UDataflow* Graph, const FString& Name);

	/**
	 * Sets the value of an existing variable using its serialized string representation.
	 * The format depends on the variable's type (e.g., "3.14" for float, "true" for bool,
	 * "42" for int, "MyName" for FName).
	 *
	 * @param Graph - The Dataflow asset to modify
	 * @param Name  - Name of the variable to set
	 * @param Value - Serialized string value to assign
	 * @return true on success, false if the variable is not found or the value cannot be parsed
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Variables")
	static bool SetVariable(UDataflow* Graph, const FString& Name, const FString& Value);

	// Comments --------------------------------------------------------------------------------------------------------

	/**
	 * Adds a comment box around the given nodes.
	 *
	 * @param Graph   - The Dataflow asset to add the comment to
	 * @param Nodes   - List of EdNodes to surround with the comment box
	 * @param Comment - Text to display on the comment box
	 * @param Color   - Background color of the comment box (defaults to White)
	 * @return Node ID string of the created comment node, or empty string on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Comments")
	static FString AddCommentBox(
		UDataflow* Graph,
		const TArray<UDataflowEdNode*>& Nodes,
		const FString& Comment,
		FLinearColor Color = FLinearColor::White);

	/**
	 * Removes a comment box node from the graph.
	 *
	 * @param Graph     - The Dataflow asset containing the comment
	 * @param CommentId - The node GUID string returned by AddCommentBox
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Comments")
	static void RemoveCommentBox(UDataflow* Graph, const FString& CommentId);

	// Assets ----------------------------------------------------------------------------------------------------------

	/**
	 * Returns a JSON list of every UClass that can host an embedded Dataflow graph
	 * (i.e. implements IDataflowInstanceInterface). Each entry has "className",
	 * "displayName", and "modulePath" fields.
	 *
	 * Use the "className" value as input to CreateDataflowCompatibleAsset or
	 * ListDataflowTemplatesForAssetClass.
	 *
	 * @return JSON array of compatible asset type descriptors
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Assets")
	static FString ListDataflowCompatibleAssetTypes();

	/**
	 * Creates a new Dataflow-compatible asset (e.g. ChaosClothAsset, GeometryCollection,
	 * FleshAsset, GroomAsset) with an empty embedded Dataflow graph.
	 *
	 * @param ClassName - Asset class name with or without the "U"/"A" prefix
	 *                    (e.g. "ChaosClothAsset" or "UChaosClothAsset")
	 * @param Name      - Name for the new asset
	 * @param Path      - Content folder path where the asset should be created. Defaults to "/Game/Dataflow"
	 * @return The full asset path of the created asset, or empty string on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Assets")
	static FString CreateDataflowCompatibleAsset(
		const FString& ClassName,
		const FString& Name,
		const FString& Path = TEXT("/Game/Dataflow"));

	/**
	 * Creates a new Dataflow-compatible asset and initialises its embedded Dataflow graph
	 * from a registered template in one step.
	 *
	 * @param ClassName  - Asset class name with or without the "U"/"A" prefix
	 * @param Name       - Name for the new asset
	 * @param Path       - Content folder path
	 * @param TemplateId - Template identifier returned by ListDataflowTemplatesForAssetClass.
	 *                     Pass an empty string to create with an empty embedded graph
	 *                     (equivalent to CreateDataflowCompatibleAsset).
	 * @return The full asset path of the created asset, or empty string on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Assets")
	static FString CreateDataflowCompatibleAssetFromTemplate(
		const FString& ClassName,
		const FString& Name,
		const FString& Path,
		const FString& TemplateId);

	// Templates -------------------------------------------------------------------------------------------------------

	/**
	 * Returns a JSON list of Dataflow templates registered for the given asset class.
	 * Templates registered for parent classes are included (class hierarchy walk).
	 *
	 * @param ClassName     - Asset class name with or without the "U"/"A" prefix
	 * @param bIncludeBlank - When true, include a "Blank" option (empty graph) at the start
	 * @return JSON array of `{ "templateId", "displayName", "tooltip" }` objects.
	 *         "templateId" is opaque - pass it back to AssignDataflowTemplate or
	 *         CreateDataflowCompatibleAssetFromTemplate.
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Templates")
	static FString ListDataflowTemplatesForAssetClass(
		const FString& ClassName,
		const bool bIncludeBlank = true);

	/**
	 * Assigns a Dataflow template to an existing Dataflow-compatible asset by duplicating
	 * the template graph and embedding it. Replaces any existing embedded graph.
	 *
	 * @param Asset      - The target asset (must implement IDataflowInstanceInterface)
	 * @param TemplateId - Template identifier returned by ListDataflowTemplatesForAssetClass.
	 *                     Pass an empty string to assign a fresh empty graph.
	 * @return true on success, false on failure
	 */
	UFUNCTION(meta=(AICallable), Category = "Dataflow|Templates")
	static bool AssignDataflowTemplate(UObject* Asset, const FString& TemplateId);
};
