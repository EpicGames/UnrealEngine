// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataflowAgentCustomTypes.generated.h"

/** Basic info about a Dataflow node returned by GetNodeInfo */
USTRUCT(BlueprintType)
struct FDataflowNodeInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	FString Name;

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	FString TypeName;

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	int32 PosX = 0;

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	int32 PosY = 0;
};

/** Describes one end of a connection in a Dataflow graph */
USTRUCT(BlueprintType)
struct FDataflowConnectionInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	FString NodeName;

	UPROPERTY(BlueprintReadOnly, Category = "Dataflow")
	FString PinName;
};
