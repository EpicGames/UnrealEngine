// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowAgentToolset.h"
#include "ToolsetRegistry/AgentSkill.h"
#include "ToolsetRegistry/ToolsetRegistry.h"

#include "DataflowAgentSkills.generated.h"

UCLASS()
class UDataflowGraphEditingSkill : public UAgentSkill
{
	GENERATED_BODY()

public:
	UDataflowGraphEditingSkill()
	{
		Description = TEXT("Skill for generating and editing Dataflow graphs (simulation / deformation nodes)");
	}

protected:
	virtual FString GeneratePrompt_Implementation(const FString& InitialInstructions) const override
	{
		// Build a simple prompt that lists available node types and forwards the initial instructions.
		FString NodeList = UDataflowAgentToolset::ListNodeTypes(true);

		FString Prompt;
		Prompt += TEXT("You are an AI assistant helping the user build and edit Unreal Engine Dataflow graphs.\n\n");
		Prompt += TEXT("You can also create Dataflow-compatible assets (e.g. ChaosClothAsset, GeometryCollection, ");
		Prompt += TEXT("FleshAsset, GroomAsset) and assign Dataflow templates to them:\n");
		Prompt += TEXT("  - ListDataflowCompatibleAssetTypes()         : enumerate asset classes that can host a Dataflow graph\n");
		Prompt += TEXT("  - ListDataflowTemplatesForAssetClass(Class)  : list registered templates compatible with an asset class\n");
		Prompt += TEXT("  - CreateDataflowCompatibleAsset(...)         : create a new asset with an empty embedded Dataflow\n");
		Prompt += TEXT("  - CreateDataflowCompatibleAssetFromTemplate(...) : create an asset preloaded with a template\n");
		Prompt += TEXT("  - AssignDataflowTemplate(Asset, TemplateId)  : (re)assign a template to an existing asset\n");
		Prompt += TEXT("Template ids are opaque strings returned by ListDataflowTemplatesForAssetClass - pass them back verbatim.\n\n");
		Prompt += TEXT("Available node types:\n");
		Prompt += NodeList;
		Prompt += TEXT("\n\n");
		Prompt += InitialInstructions;
		return Prompt;
	}
};
